// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64", "Linux")]
public class NetworkPredictionTests : TestModuleRules
{
	static NetworkPredictionTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "NetworkPredictionPlugin";
			TestMetadata.TestShortName = "Net Prediction";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
		}
	}

	public NetworkPredictionTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"NetworkPrediction",
			}
		);
	}
}
