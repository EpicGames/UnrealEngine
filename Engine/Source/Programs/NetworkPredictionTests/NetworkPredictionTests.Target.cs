// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64", "Linux")]
public class NetworkPredictionTestsTarget : TestTargetRules
{
	public NetworkPredictionTestsTarget(TargetInfo Target) : base(Target)
	{
		bTestsRequireEngine = true;
		bTestsRequireApplicationCore = true;
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;

		bUsesSlate = false;

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
	}
}
