// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AssertionMacros.h"
#include "NetPredictionMockPackageMap.h"

UNetPredictionMockPackageMap* UNetPredictionMockPackageMap::Get()
{
	static UNetPredictionMockPackageMap* Instance = nullptr;
	if (Instance == nullptr)
	{
		Instance = NewObject<UNetPredictionMockPackageMap>();
	}
	return Instance;
}

bool UNetPredictionMockPackageMap::WriteObject(FArchive & Ar, UObject* InOuter, FNetworkGUID NetGUID, FString ObjName)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return false;
}

bool UNetPredictionMockPackageMap::SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID *OutNetGUID)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return false;
}

bool UNetPredictionMockPackageMap::SerializeName(FArchive& Ar, FName& InName)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return false;
}

UObject* UNetPredictionMockPackageMap::ResolvePathAndAssignNetGUID(const FNetworkGUID& NetGUID, const FString& PathName)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return nullptr;
}

bool UNetPredictionMockPackageMap::SerializeNewActor(FArchive & Ar, class UActorChannel * Channel, class AActor *& Actor)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return false;
}

UObject* UNetPredictionMockPackageMap::GetObjectFromNetGUID(const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped )
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return nullptr;
}

FNetworkGUID UNetPredictionMockPackageMap::GetNetGUIDFromObject(const UObject* InObject) const
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
	return FNetworkGUID();
}

void UNetPredictionMockPackageMap::Serialize(FArchive& Ar)
{
	ensureAlwaysMsgf(false, TEXT("Not supported"));
}
