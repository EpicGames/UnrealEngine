// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/CoreNet.h"
#include "NetPredictionMockPackageMap.generated.h"

// Placeholder package map that ensures if any of its functionality is actually used.
UCLASS(transient)
class UNetPredictionMockPackageMap : public UPackageMap
{
	GENERATED_BODY()

public:
	virtual bool WriteObject(FArchive & Ar, UObject* InOuter, FNetworkGUID NetGUID, FString ObjName) override;
	virtual bool SerializeObject(FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID *OutNetGUID = NULL) override;
	virtual bool SerializeName(FArchive& Ar, FName& InName) override;
	virtual UObject* ResolvePathAndAssignNetGUID(const FNetworkGUID& NetGUID, const FString& PathName) override;
	virtual bool SerializeNewActor(FArchive & Ar, class UActorChannel * Channel, class AActor *& Actor) override;
	virtual UObject* GetObjectFromNetGUID(const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped) override;
	virtual FNetworkGUID GetNetGUIDFromObject(const UObject* InObject) const override;
	virtual void Serialize(FArchive& Ar) override;

	static UNetPredictionMockPackageMap* Get();
};
