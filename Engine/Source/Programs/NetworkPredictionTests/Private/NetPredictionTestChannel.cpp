// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetPredictionTestChannel.h"
#include "UObject/CoreNet.h"

namespace UE::Net::Private
{

void FNetPredictionTestChannel::Send(const FNetBitWriter& Writer)
{
	DataBuffer.Enqueue(Writer);
}

bool FNetPredictionTestChannel::HasPendingData()
{
	return !DataBuffer.IsEmpty();
}

FNetBitReader FNetPredictionTestChannel::Receive()
{
	const FNetBitWriter& Writer = DataBuffer.Peek();
	FNetBitReader Reader(nullptr, Writer.GetData(), Writer.GetNumBits());

	DataBuffer.Pop();

	return MoveTemp(Reader);
}

}