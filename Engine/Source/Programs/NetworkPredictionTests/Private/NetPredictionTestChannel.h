// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "UObject/CoreNet.h"

namespace UE::Net::Private
{

struct FNetPredictionTestChannel
{
	void Send(const FNetBitWriter& Writer);
	bool HasPendingData();
	FNetBitReader Receive();

private:
	TResizableCircularQueue<FNetBitWriter> DataBuffer;
};

}