// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetPredictionTestDriver.h"
#include "HAL/PlatformMath.h"
#include "NetPredictionMockPackageMap.h"
#include "NetPredictionTestChannel.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "NetworkPredictionProxyInit.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionTests.h"

#include <catch2/catch_test_macros.hpp>

namespace UE::Net::Private
{

const uint8 FNetPredictionTestDriver::ChangeFlagsBitsNeeded = 1 + FPlatformMath::FloorLog2(static_cast<uint8>(EChangeFlags::MaxFlagPlusOne) - 1);

struct FTestNetPredictionModelDef : public FNetworkPredictionModelDef
{
	NP_MODEL_BODY();

	using Simulation = FNetPredictionTestDriver;
	using StateTypes = FNetPredictionTestStateTypes;
	using Driver = FNetPredictionTestDriver;

	static const TCHAR* GetName() { return TEXT("TestModelDef"); }
};

NP_MODEL_REGISTER(FTestNetPredictionModelDef);

bool FNetPredictionTestSyncState::ShouldReconcile(const FNetPredictionTestSyncState& AuthorityState) const
{
	return AutoCounter != AuthorityState.AutoCounter || InputCounter != AuthorityState.InputCounter;
}

void FNetPredictionTestSyncState::Interpolate(const FNetPredictionTestSyncState* From, const FNetPredictionTestSyncState* To, float Percent)
{
	UE_LOGF(LogNetworkPredictionTests, Verbose, "Interpolate: from {%.2f, %.2f} to {%.2f, %.2f}, pct %.2f.",
		From->AutoCounter, From->InputCounter, To->AutoCounter, To->InputCounter, Percent);

	AutoCounter = FMath::Lerp(From->AutoCounter, To->AutoCounter, Percent);
	InputCounter = FMath::Lerp(From->InputCounter, To->InputCounter, Percent);
}

FNetPredictionTestDriver::FNetPredictionTestDriver(UNetworkPredictionWorldManager* WorldManager, ENetMode Mode,
	TSharedPtr<FNetPredictionTestChannel> InClientToServer,
	TSharedPtr<FNetPredictionTestChannel> InServerToClient)
	: ClientToServer(InClientToServer)
	, ServerToClient(InServerToClient)
{
	ReplicationProxy_ServerRPC.Init(&Proxy, EReplicationProxyTarget::ServerRPC);
	ReplicationProxy_Autonomous.Init(&Proxy, EReplicationProxyTarget::AutonomousProxy);
	ReplicationProxy_Simulated.Init(&Proxy, EReplicationProxyTarget::SimulatedProxy);

	FNetworkPredictionProxy::FInitParams<FTestNetPredictionModelDef> Params = {WorldManager, Mode, GetReplicationProxies(), this, this};
	Proxy.Init<FTestNetPredictionModelDef>(Params);
}

void FNetPredictionTestDriver::ProduceInput(const int32 DeltaTimeMS, FNetPredictionTestInputCmd* Cmd)
{
	UE_LOGF(LogNetworkPredictionTests, Verbose, "ProduceInput: %ls. SimFrame: %d", *DebugName, Proxy.GetPendingFrame());

	Cmd->bIncrement = bInputPressed;
}

void FNetPredictionTestDriver::SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<FNetPredictionTestStateTypes>& SimInput, const TNetSimOutput<FNetPredictionTestStateTypes>& SimOutput)
{
	UE_LOGF(LogNetworkPredictionTests, Verbose, "SimulationTick: %ls. Frame: %d", *DebugName, TimeStep.Frame);

	SimOutput.Sync->InputCounter = SimInput.Sync->InputCounter;
	if (SimInput.Cmd->bIncrement)
	{
		SimOutput.Sync->InputCounter += 1.0f;
	}

	SimOutput.Sync->AutoCounter = SimInput.Sync->AutoCounter + 1.0f;
}

void FNetPredictionTestDriver::InitializeSimulationState(FNetPredictionTestSyncState* OutSync, void* OutAux)
{
	UE_LOGF(LogNetworkPredictionTests, Verbose, "InitializeSimulationState: %ls", *DebugName);
	OutSync->AutoCounter = 0.0f;
	OutSync->InputCounter = 0.0f;
}

void FNetPredictionTestDriver::FinalizeFrame(const FNetPredictionTestSyncState* SyncState, const void* AuxState)
{
	UE_LOGF(LogNetworkPredictionTests, Verbose, "FinalizeFrame: %ls. AutoCounter: %.2f, InputCounter: %.2f", *DebugName, SyncState->AutoCounter, SyncState->InputCounter);
}

void FNetPredictionTestDriver::SetHiddenForInterpolation(bool bInHidden)
{
	bHidden = bInHidden;
}

void FNetPredictionTestDriver::CallServerRPC()
{
	FServerReplicationRPCParameter ProxyParameter(ReplicationProxy_ServerRPC);

	FNetBitWriter Ar(1024 * 8 * 2);
	bool bOutSuccess = false;
	ProxyParameter.SerializeToProxy(Ar, UNetPredictionMockPackageMap::Get(), bOutSuccess);

	ClientToServer->Send(Ar);
}

void FNetPredictionTestDriver::ReceiveServerRPCs()
{
	while (ClientToServer->HasPendingData())
	{
		FNetBitReader Ar = ClientToServer->Receive();

		bool bOutSuccess = false;
		ReplicationProxy_ServerRPC.NetSerialize(Ar, UNetPredictionMockPackageMap::Get(), bOutSuccess);
	}
}

void FNetPredictionTestDriver::NetSerialize(FArchive& Ar, EChangeFlags Flags)
{
	bool bSuccess = false;

	Ar.SerializeBits(&Flags, ChangeFlagsBitsNeeded);

	if (EnumHasAnyFlags(Flags, EChangeFlags::PredictionProxyChanged))
	{
		Proxy.NetSerialize(Ar, UNetPredictionMockPackageMap::Get(), bSuccess);
	}

	if (EnumHasAnyFlags(Flags, EChangeFlags::ReplicationProxyChanged))
	{
		// Emulate the replication conditions from UNetworkPredictionComponent
		if (Proxy.GetCachedHasNetConnection())
		{
			ReplicationProxy_Autonomous.NetSerialize(Ar, UNetPredictionMockPackageMap::Get(), bSuccess);
		}
		else
		{
			ReplicationProxy_Simulated.NetSerialize(Ar, UNetPredictionMockPackageMap::Get(), bSuccess);
		}
	}
}

FNetPredictionTestObject::FNetPredictionTestObject(
		UNetworkPredictionWorldManager* ServerWorldManager,
		UNetworkPredictionWorldManager* ClientWorldManager,
		ENetRole ClientRole)
	: ClientToServer(MakeShared<FNetPredictionTestChannel>())
	, ServerToClient(MakeShared<FNetPredictionTestChannel>())
	, ServerObject(ServerWorldManager, NM_DedicatedServer, ClientToServer, ServerToClient)
	, ClientObject(ClientWorldManager, NM_Client, ClientToServer, ServerToClient)
{
	ServerObject.Proxy.InitForNetworkRole(ENetRole::ROLE_Authority, ClientRole == ENetRole::ROLE_AutonomousProxy);
	ClientObject.Proxy.InitForNetworkRole(ClientRole, ClientRole == ENetRole::ROLE_AutonomousProxy);
}

void FNetPredictionTestObject::ServerSend()
{
	ServerObject.ReplicationProxy_Autonomous.OnPreReplication();
	ServerObject.ReplicationProxy_Simulated.OnPreReplication();
	
	// Simple emulation of not sending changed properties.
	// Note this only works when no "packets" are "dropped" in the tests, which is currently the case
	FNetPredictionTestDriver::EChangeFlags Flags = FNetPredictionTestDriver::EChangeFlags::None;
	
	if (!ProxyShadowState.Identical(&ServerObject.Proxy, 0))
	{
		Flags |= FNetPredictionTestDriver::EChangeFlags::PredictionProxyChanged;
		ProxyShadowState = ServerObject.Proxy;
	}

	// Emulate the replication conditions from UNetworkPredictionComponent
	if (ServerObject.Proxy.GetCachedHasNetConnection())
	{
		if (!AutonomousProxyShadowState.Identical(&ServerObject.ReplicationProxy_Autonomous, 0))
		{
			Flags |= FNetPredictionTestDriver::EChangeFlags::ReplicationProxyChanged;
			AutonomousProxyShadowState = ServerObject.ReplicationProxy_Autonomous;
		}
	}
	else
	{
		if (!SimulatedProxyShadowState.Identical(&ServerObject.ReplicationProxy_Simulated, 0))
		{
			Flags |= FNetPredictionTestDriver::EChangeFlags::ReplicationProxyChanged;
			SimulatedProxyShadowState = ServerObject.ReplicationProxy_Simulated;
		}
	}

	FNetBitWriter TempWriter(1024 * 8 * 2);
	ServerObject.NetSerialize(TempWriter, Flags);
	ServerToClient->Send(TempWriter);
}

void FNetPredictionTestObject::ClientReceive()
{
	while(ServerToClient->HasPendingData())
	{
		FNetBitReader TempReader = ServerToClient->Receive();
		ClientObject.NetSerialize(TempReader, FNetPredictionTestDriver::EChangeFlags());
	}
}

}