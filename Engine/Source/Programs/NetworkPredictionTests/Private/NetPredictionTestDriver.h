// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Misc/EnumClassFlags.h"
#include "NetPredictionTestChannel.h"
#include "NetworkPredictionModelDef.h"
#include "NetworkPredictionProxy.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionSimulation.h"
#include "NetworkPredictionTickState.h"

class FArchive;
class UNetworkPredictionWorldManager;

namespace UE::Net::Private
{

struct FNetPredictionTestInputCmd
{
	bool bIncrement = false;

	void ToString(FAnsiStringBuilderBase& Builder) const
	{
		Builder << TEXT("bIncrement: ") << bIncrement;
	}

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar.SerializeBits(&bIncrement, 1);
	}
};

struct FNetPredictionTestSyncState
{
	// Value is incremented by one every frame, not dependent on input
	float AutoCounter = 0.0f;

	// Value is incremented by one if the input bIncrement value is true, no change if it's false.
	float InputCounter = 0.0f;

	void ToString(FAnsiStringBuilderBase& Builder) const
	{
		Builder << FString::Printf(TEXT("AutoCounter: %.2f, InputCounter: %.2f"), AutoCounter, InputCounter);
	}

	void NetSerialize(const FNetSerializeParams& P)
	{
		P.Ar << AutoCounter;
		P.Ar << InputCounter;
	}

	void Interpolate(const FNetPredictionTestSyncState* From, const FNetPredictionTestSyncState* To, float Percent);
	bool ShouldReconcile(const FNetPredictionTestSyncState& AuthorityState) const;
};

using FNetPredictionTestStateTypes = TNetworkPredictionStateTypes<FNetPredictionTestInputCmd, FNetPredictionTestSyncState, void>;

// Mimics the key parts of UNetworkPredictionComponent without being an actual UActorComponent.
struct FNetPredictionTestDriver
{
	FNetworkPredictionProxy Proxy;
	FReplicationProxy ReplicationProxy_ServerRPC;
	FReplicationProxy ReplicationProxy_Autonomous;
	FReplicationProxy ReplicationProxy_Simulated;

	FString DebugName;

	bool bHidden = false;
	bool bInputPressed = false;

	enum class EChangeFlags : uint8
	{
		None = 0,
		PredictionProxyChanged = 1 << 0,
		ReplicationProxyChanged = 1 << 1,

		// Add new entries above
		MaxFlagPlusOne
	};
	static const uint8 ChangeFlagsBitsNeeded;

	FNetPredictionTestDriver(UNetworkPredictionWorldManager* WorldManager, ENetMode Mode,
		TSharedPtr<FNetPredictionTestChannel> InClientToServer,
		TSharedPtr<FNetPredictionTestChannel> InServerToClient);

	void ProduceInput(const int32 DeltaTimeMS, FNetPredictionTestInputCmd* Cmd);
	void SimulationTick(const FNetSimTimeStep& TimeStep, const TNetSimInput<FNetPredictionTestStateTypes>& SimInput, const TNetSimOutput<FNetPredictionTestStateTypes>& SimOutput);
	void InitializeSimulationState(FNetPredictionTestSyncState* OutSync, void* OutAux);
	void FinalizeFrame(const FNetPredictionTestSyncState* SyncState, const void* AuxState);
	void SetHiddenForInterpolation(bool bInHidden);
	void CallServerRPC();
	void ReceiveServerRPCs();

	void NetSerialize(FArchive& Ar, EChangeFlags Flags);

private:
	FReplicationProxySet GetReplicationProxies()
	{
		return FReplicationProxySet{&ReplicationProxy_ServerRPC, &ReplicationProxy_Autonomous, &ReplicationProxy_Simulated, nullptr};
	}

	TSharedPtr<FNetPredictionTestChannel> ClientToServer;
	TSharedPtr<FNetPredictionTestChannel> ServerToClient;
};

ENUM_CLASS_FLAGS(FNetPredictionTestDriver::EChangeFlags);

// Conveniently associates the server & client object drivers and their channels for ease of writing unit tests.
struct FNetPredictionTestObject
{
	TSharedPtr<FNetPredictionTestChannel> ClientToServer;
	TSharedPtr<FNetPredictionTestChannel> ServerToClient;

	FNetPredictionTestDriver ServerObject;
	FNetPredictionTestDriver ClientObject;

	FNetPredictionTestObject(
		UNetworkPredictionWorldManager* ServerWorldManager,
		UNetworkPredictionWorldManager* ClientWorldManager,
		ENetRole ClientRole);

	void ServerSend();
	void ClientReceive();

private:
	// Stores previously sent state, compared with current state to prevent "sending" when it hasn't changed.
	// For emulating FRepLayout behavior.
	FNetworkPredictionProxy ProxyShadowState;
	FReplicationProxy AutonomousProxyShadowState;
	FReplicationProxy SimulatedProxyShadowState;
};

}