// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetPredictionTestWorld.h"
#include "NetworkPredictionWorldManager.h"

namespace UE::Net::Private
{

FNetPredictionTestWorld::FNetPredictionTestWorld(UNetworkPredictionSettingsObject* Settings, ENetMode InNetMode)
	: NetMode(InNetMode)
{
	WorldManager = NewObject<UNetworkPredictionWorldManager>();
	if (Settings)
	{
		WorldManager->SyncNetworkPredictionSettings(Settings);
	}
}

void FNetPredictionTestWorld::PreTick(int32 FixedFrameRate)
{
	checkf(FixedFrameRate > 0, TEXT("FixedFrameRate must be > 0"));
	WorldManager->OnWorldPreTick_Internal(1.0f / FixedFrameRate, FixedFrameRate);
}

void FNetPredictionTestWorld::Tick(int32 FixedFrameRate)
{
	checkf(FixedFrameRate > 0, TEXT("FixedFrameRate must be > 0"));
	if (NetMode == NM_Client)
	{
		WorldManager->ReconcileSimulationsPostNetworkUpdate_Internal();
	}
	WorldManager->BeginNewSimulationFrame_Internal(1.0f / FixedFrameRate);
}

}