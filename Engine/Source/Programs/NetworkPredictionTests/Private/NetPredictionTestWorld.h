// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"

class UNetworkPredictionWorldManager;
class UNetworkPredictionSettingsObject;

namespace UE::Net::Private
{

struct FNetPredictionTestWorld
{
public:
	FNetPredictionTestWorld(UNetworkPredictionSettingsObject* Settings, ENetMode InNetMode);

	void PreTick(int32 FixedFrameRate);
	void Tick(int32 FixedFrameRate);

	UNetworkPredictionWorldManager* WorldManager = nullptr;

private:
	ENetMode NetMode;
};

}