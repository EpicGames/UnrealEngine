// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionCues.h"
#include "NetworkPredictionModelDef.h"
#include "NetworkPredictionDriver.h"
#include "Math/Vector.h"
#include "Misc/Optional.h"
#include "UObject/CoreNet.h"

#include <catch2/catch_test_macros.hpp>

namespace UE::Net::Private
{

// ------------------------------------------------------------------------------------------------------------------------------------------
//	Mock Cue example: Minimal example of NetSimCue pipeline
//		(1)	- How to define cue types and traits
//		(2) - How to define cue sets
//		(3) - How to define handlers
//		(4) - How to invoke cues in simulation
//
// ------------------------------------------------------------------------------------------------------------------------------------------

// ---------------------------------------------------------------------
// (1) Mock Cue Types
// ---------------------------------------------------------------------

// Example "Weak" cue (will default to NetSimCueTraits::Default traits, which are "Weak" non replicated, non resimulated). No NetSerialize/NetIdentical is required.
struct FMockImpactCue
{
	NETSIMCUE_BODY();

	FMockImpactCue() = default;
	FMockImpactCue(const FVector& InImpact) : ImpactLocation(InImpact) { }

	FVector ImpactLocation;

};
NETSIMCUE_REGISTER(FMockImpactCue, TEXT("Impact")); // Must go in .cpp
static_assert(TNetSimCueTraits<FMockImpactCue>::InvokeMask == NetSimCueTraits::Default::InvokeMask, "Traits Errors"); // Not required: just asserting what this classes traits are

// ----------------

// Example "Strong" cue. Explicitly sets NetSimCueTraits::Strong and implements NetSerialize/NetIdentical (will not compile without these functions) 
struct FMockDamageCue
{
	NETSIMCUE_BODY();

	FMockDamageCue() = default;
	FMockDamageCue(const int32 InSourceID, const int32 InDamageType, const FVector& InHitLocation) 
		: SourceID(InSourceID), DamageType(InDamageType), HitLocation(InHitLocation) { }

	int32 SourceID;
	int32 DamageType;
	FVector HitLocation;

	using Traits = NetSimCueTraits::Strong; // Sets our traits via preset.
	
	void NetSerialize(FArchive& Ar)
	{
		Ar << SourceID;
		Ar << DamageType;
		Ar << HitLocation;
	}

	bool NetIdentical(const FMockDamageCue& Other) const
	{
		const float ErrorTolerance = 1.f;
		return SourceID == Other.SourceID && DamageType && Other.DamageType && HitLocation.Equals(Other.HitLocation, ErrorTolerance);
	}
};
NETSIMCUE_REGISTER(FMockDamageCue, TEXT("Damage")); // Must go in .cpp
static_assert(TNetSimCueTraits<FMockDamageCue>::InvokeMask == NetSimCueTraits::Strong::InvokeMask, "Traits Errors"); // Not required: just asserting what this classes traits are

// Alternative way of explicitly setting traits
struct FMockHealingCue
{
	NETSIMCUE_BODY();
	
	FMockHealingCue() = default;
	FMockHealingCue(const int32 InSourceID, const int32 InHealingAmount) 
		: SourceID(InSourceID), HealingAmount(InHealingAmount) { }

	int32 SourceID;
	int32 HealingAmount;

	using Traits = TNetSimCueTraitsExplicit<ENetSimCueInvoker::All, ENetSimCueReplicationTarget::All, true>; // Sets our traits explicitly
	
	void NetSerialize(FArchive& Ar)
	{
		Ar << SourceID;
		Ar << HealingAmount;
	}

	bool NetIdentical(const FMockHealingCue& Other) const
	{
		const float ErrorTolerance = 1.f;
		return SourceID == Other.SourceID && HealingAmount == Other.HealingAmount;
	}
};
NETSIMCUE_REGISTER(FMockHealingCue, TEXT("Healing")); // Must go in .cpp
static_assert(TNetSimCueTraits<FMockHealingCue>::InvokeMask == ENetSimCueInvoker::All, "Traits Errors"); // Not required: just asserting what this classes traits are

// ---------------------------------------------------------------------
// (2) Cue Sets: just a collection of cue types that a simulation emits.
//
//	This isn't really a core type, it is just a pattern for grouping cue types
//	together so that handlers can register with sets of cues rather than each cue individually
//
//	This example illustrates how a base set can be built off of by child simulations
// ---------------------------------------------------------------------

struct FMockCueSet_Base
{
	template<typename TDispatchTable>
	static void RegisterNetSimCueTypes(TDispatchTable& DispatchTable)
	{
		DispatchTable.template RegisterType<FMockImpactCue>();
	}
};

struct FMockCueSet_Child
{
	template<typename TDispatchTable>
	static void RegisterNetSimCueTypes(TDispatchTable& DispatchTable)
	{
		FMockCueSet_Base::RegisterNetSimCueTypes(DispatchTable); // Superset of the base cueset
		DispatchTable.template RegisterType<FMockDamageCue>();
		DispatchTable.template RegisterType<FMockHealingCue>();
	}
};


// ---------------------------------------------------------------------
// (3) Mock Handlers 
//	-Implements ::HandeCue function to receive the cue (thats it!)
//	-Meant to be implemented by owning actor/component that is driving the NetworkedSimulation
//	-E.g, in practice, these aren't standalone classes. These are just functions/body that you put on your actor/component.
//
//	Again, we show how a base handler can be implemented and how a child class can extend off of that.
// ---------------------------------------------------------------------

class TestHandlerBase
{
public:
	// Store handled cue so the test code can verify it
	TOptional<FMockImpactCue> HandledImpact;

	void HandleCue(const FMockImpactCue& ImpactCue, const FNetSimCueSystemParamemters& SystemParameters)
	{
		HandledImpact = ImpactCue;
	}
};

NETSIMCUESET_REGISTER(TestHandlerBase, FMockCueSet_Base); // Must go in cpp. Says "TestHandlerBase" can accepts cues in "FMockCueSet_Base"

class TestHandlerChild : public TestHandlerBase
{
public:

	using TestHandlerBase::HandleCue; // Required for us to "use" the HandleCue methods in our parent class

	// Store handled cues so the test code can verify them
	TOptional<FMockDamageCue> HandledDamage;
	TOptional<FMockHealingCue> HandledHealing;

	void HandleCue(const FMockDamageCue& DamageCue, const FNetSimCueSystemParamemters& SystemParameters)
	{
		HandledDamage = DamageCue;
	}

	void HandleCue(const FMockHealingCue& HealingCue, const FNetSimCueSystemParamemters& SystemParameters)
	{
		HandledHealing = HealingCue;
	}
};

NETSIMCUESET_REGISTER(TestHandlerChild, FMockCueSet_Child); // Must go in cpp. Says "TestHandlerBase" can accepts cues in "FMockCueSet_Base" // Must go in cpp. Says "TestHandlerBase" can accepts cues in "FMockCueSet_Base"

// -------------------------------------
// (4) Simuation/Invoke
// -------------------------------------
const FVector ImpactLocation{1.f, 2.f, 3.f};

const int32 DamageSourceID = 1;
const int32 DamageType = 2;
const FVector DamageHitLocation{4.f, 5.f, 6.f};
	
const int32 HealingSourceID = 10;
const int32 HealingAmount = 32;

struct FBaseSimulation
{
	static void TickSimulation(FNetSimCueDispatcher& Dispatcher)
	{
		Dispatcher.Invoke<FMockImpactCue>(ImpactLocation);
	}
};

struct FChildSimulation
{
	static void TickSimulation(FNetSimCueDispatcher& Dispatcher)
	{
		FBaseSimulation::TickSimulation(Dispatcher);

		Dispatcher.Invoke<FMockDamageCue>(DamageSourceID, DamageType, DamageHitLocation);
		Dispatcher.Invoke<FMockHealingCue>(HealingSourceID, HealingAmount);
	}
};

TEST_CASE("NetworkPrediction basic cue functionality")
{
	const int32 SimFrameNum = 1;
	const int32 SimFrameTimeMS = 16;

	TestHandlerChild ServerHandlerObject;

	TNetSimCueDispatcher<FNetworkPredictionModelDef> ServerDispatcher;

	ServerDispatcher.PushContext({SimFrameNum, SimFrameTimeMS, ESimulationTickContext::Authority});
	FChildSimulation::TickSimulation(ServerDispatcher);

	// No cues should have been handled on the server yet
	REQUIRE_FALSE(ServerHandlerObject.HandledImpact.IsSet());
	REQUIRE_FALSE(ServerHandlerObject.HandledDamage.IsSet());
	REQUIRE_FALSE(ServerHandlerObject.HandledHealing.IsSet());

	ServerDispatcher.DispatchCueRecord(ServerHandlerObject, SimFrameNum, SimFrameTimeMS, SimFrameTimeMS);
	ServerDispatcher.PopContext();

	// Server should have handled all three cues
	REQUIRE(ServerHandlerObject.HandledImpact.IsSet());
	REQUIRE(ServerHandlerObject.HandledImpact->ImpactLocation.Equals(ImpactLocation, 0.1f));

	REQUIRE(ServerHandlerObject.HandledDamage.IsSet());
	REQUIRE(ServerHandlerObject.HandledDamage->SourceID == DamageSourceID);
	REQUIRE(ServerHandlerObject.HandledDamage->DamageType == DamageType);
	REQUIRE(ServerHandlerObject.HandledDamage->HitLocation.Equals(DamageHitLocation, 0.1f));

	REQUIRE(ServerHandlerObject.HandledHealing.IsSet());
	REQUIRE(ServerHandlerObject.HandledHealing->SourceID == HealingSourceID);
	REQUIRE(ServerHandlerObject.HandledHealing->HealingAmount == HealingAmount);
	
	// -------------------------------------
	// Send
	// -------------------------------------

	FNetBitWriter TempWriter(1024 << 3);
	ServerDispatcher.NetSendSavedCues(TempWriter, ENetSimCueReplicationTarget::All, true);

	// -------------------------------------
	// Receive
	// -------------------------------------
	TestHandlerChild ClientHandlerObject;

	FNetBitReader TempReader(nullptr, TempWriter.GetData(), TempWriter.GetNumBits());
	TNetSimCueDispatcher<FNetworkPredictionModelDef> ClientDispatcher;
	ClientDispatcher.SetReceiveReplicationTarget(ENetSimCueReplicationTarget::All);
	ClientDispatcher.NetRecvSavedCues(TempReader, true, 0, 0);

	// No cues should have been handled on the client yet
	REQUIRE_FALSE(ClientHandlerObject.HandledImpact.IsSet());
	REQUIRE_FALSE(ClientHandlerObject.HandledDamage.IsSet());
	REQUIRE_FALSE(ClientHandlerObject.HandledHealing.IsSet());

	ClientDispatcher.DispatchCueRecord(ClientHandlerObject, SimFrameNum, SimFrameTimeMS, SimFrameTimeMS);

	// Client should have handled only damage & healing, impact is not replicated
	REQUIRE_FALSE(ClientHandlerObject.HandledImpact.IsSet());

	REQUIRE(ClientHandlerObject.HandledDamage.IsSet());
	REQUIRE(ClientHandlerObject.HandledDamage->SourceID == DamageSourceID);
	REQUIRE(ClientHandlerObject.HandledDamage->DamageType == DamageType);
	REQUIRE(ClientHandlerObject.HandledDamage->HitLocation.Equals(DamageHitLocation, 0.1f));

	REQUIRE(ClientHandlerObject.HandledHealing.IsSet());
	REQUIRE(ClientHandlerObject.HandledHealing->SourceID == HealingSourceID);
	REQUIRE(ClientHandlerObject.HandledHealing->HealingAmount == HealingAmount);
}

}