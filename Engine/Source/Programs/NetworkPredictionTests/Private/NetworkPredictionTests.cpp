// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/PlatformMisc.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "NetworkPredictionTests.h"
#include "SlateGlobals.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "TestCommon/Initialization.h"

#include <catch2/catch_test_macros.hpp>

DEFINE_LOG_CATEGORY(LogNetworkPredictionTests);

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	// Add command-line options always needed for the tests
	FCommandLine::Append(TEXT(" -nullrhi -unattended"));

	{
		// Find the engine directory before InitAll() enables the platform file stub. Prevents warnings.
		FPlatformMisc::EngineDir();

		// Silence some warnings during initialization unrelated to the tests
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlate, ELogVerbosity::Error);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlateStyle, ELogVerbosity::Error);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogUObjectGlobals, ELogVerbosity::Fatal);
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogStreaming, ELogVerbosity::Error);
		InitAll(true, true);
	}

	FModuleManager::Get().LoadModule(TEXT("NetworkPrediction"));
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}
