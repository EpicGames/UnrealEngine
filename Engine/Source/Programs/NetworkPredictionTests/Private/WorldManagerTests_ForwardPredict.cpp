// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetPredictionTestDriver.h"
#include "NetPredictionTestWorld.h"
#include "NetworkPredictionDriver.h"
#include "NetworkPredictionSerialization.h"
#include "NetworkPredictionProxy.h"

#include <catch2/catch_test_macros.hpp>

namespace UE::Net::Private
{

constexpr int32 PredictionTestFixedFrameRate = 60;

TEST_CASE("NetworkPrediction, single authority object")
{
	FNetPredictionTestWorld TestWorld(nullptr, NM_DedicatedServer);

	FNetPredictionTestDriver SimObject(TestWorld.WorldManager, NM_DedicatedServer, nullptr, nullptr);
	SimObject.Proxy.InitForNetworkRole(ENetRole::ROLE_Authority, false);

	float ExpectedAutoCounterValue = 0.0f;

	const FNetPredictionTestSyncState* TestState = SimObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
	REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);

	SECTION("Simulation advances on ticks")
	{
		for (int i = 0; i < 10; ++i)
		{
			TestWorld.PreTick(PredictionTestFixedFrameRate);
			TestWorld.Tick(PredictionTestFixedFrameRate);
			ExpectedAutoCounterValue += 1.0f;

			TestState = SimObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);
		}
	}

	SECTION("Simulation processes input")
	{
		float ExpectedInputCounterValue = 0.0f;

		REQUIRE(TestState->InputCounter == ExpectedInputCounterValue);

		// Some frames with input actions
		SimObject.bInputPressed = true;
		for (int i = 0; i < 10; ++i)
		{
			TestWorld.PreTick(PredictionTestFixedFrameRate);
			TestWorld.Tick(PredictionTestFixedFrameRate);
			ExpectedAutoCounterValue += 1.0f;
			ExpectedInputCounterValue += 1.0f;

			TestState = SimObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);
			REQUIRE(TestState->InputCounter == ExpectedInputCounterValue);
		}

		// Some frames without input actions
		SimObject.bInputPressed = false;
		for (int i = 0; i < 10; ++i)
		{
			TestWorld.PreTick(PredictionTestFixedFrameRate);
			TestWorld.Tick(PredictionTestFixedFrameRate);
			ExpectedAutoCounterValue += 1.0f;

			TestState = SimObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);
			REQUIRE(TestState->InputCounter == ExpectedInputCounterValue);
		}

		// And some more with input actions
		SimObject.bInputPressed = true;
		for (int i = 0; i < 10; ++i)
		{
			TestWorld.PreTick(PredictionTestFixedFrameRate);
			TestWorld.Tick(PredictionTestFixedFrameRate);
			ExpectedAutoCounterValue += 1.0f;
			ExpectedInputCounterValue += 1.0f;

			TestState = SimObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);
			REQUIRE(TestState->InputCounter == ExpectedInputCounterValue);
		}
	}
}

struct FSingleSimulatedObjectWorld
{
	FNetPredictionTestWorld ServerWorld;
	FNetPredictionTestWorld ClientWorld;

	FNetPredictionTestObject SimObject;

	FSingleSimulatedObjectWorld(ENetRole ClientRole)
		: ServerWorld(nullptr, NM_DedicatedServer)
		, ClientWorld(nullptr, NM_Client)
		, SimObject(ServerWorld.WorldManager, ClientWorld.WorldManager, ClientRole)
	{
	}

	void TickServer()
	{
		ServerWorld.PreTick(PredictionTestFixedFrameRate);
		SimObject.ServerObject.ReceiveServerRPCs();
		ServerWorld.Tick(PredictionTestFixedFrameRate);
		SimObject.ServerSend();
	}

	void TickClient()
	{
		ClientWorld.PreTick(PredictionTestFixedFrameRate);
		SimObject.ClientReceive();
		ClientWorld.Tick(PredictionTestFixedFrameRate);
	}
};

// For this test currently the client will roll back every frame. Looks like the system currently relies
// on having an automonous proxy to sync frame numbers. Disabled until that's addressed.
TEST_CASE("NetworkPrediction, one sim proxy object updates sim state on client", "[.][warnings]")
{
	constexpr ENetRole ClientRole = ENetRole::ROLE_SimulatedProxy;

	FSingleSimulatedObjectWorld TestWorld(ENetRole::ROLE_SimulatedProxy);

	float ExpectedAutoCounterValue = 0.0f;

	const FNetPredictionTestSyncState* TestState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
	REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);

	SECTION("Simulation advances on ticks")
	{
		for (int i = 0; i < 50; ++i)
		{
			TestWorld.TickServer();
			ExpectedAutoCounterValue += 1.0f;

			TestState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(TestState->AutoCounter == ExpectedAutoCounterValue);

			TestWorld.TickClient();

			const FNetPredictionTestSyncState* ClientState = TestWorld.SimObject.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			REQUIRE(ClientState->AutoCounter == ExpectedAutoCounterValue);
		}
	}
}

TEST_CASE("NetworkPrediction, one autonomous proxy object updates sim state on client & server")
{
	// Disabling time dilation for this test since it dynamically affects how far ahead the client gets. Restored at end.
	const bool bDefaultEnableTimeDilation = NetworkPredictionCVars::EnableTimeDilation->GetBool();
	NetworkPredictionCVars::EnableTimeDilation->Set(false);
	ON_SCOPE_EXIT { NetworkPredictionCVars::EnableTimeDilation->Set(bDefaultEnableTimeDilation); };

	constexpr ENetRole ClientRole = ENetRole::ROLE_AutonomousProxy;
	FSingleSimulatedObjectWorld TestWorld(ENetRole::ROLE_AutonomousProxy);

	float ExpectedServerAutoCounterValue = 0.0f;

	const FNetPredictionTestSyncState* ServerState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
	REQUIRE(ServerState->AutoCounter == ExpectedServerAutoCounterValue);

	// Run some frames to let things settle, will get the client ahead
	// of the server by 6 frames
	for (int i = 0; i < 10; ++i)
	{
		TestWorld.TickServer();
		ExpectedServerAutoCounterValue += 1.0f;

		ServerState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
		CHECK(ServerState->AutoCounter == ExpectedServerAutoCounterValue);
		CHECK(ServerState->InputCounter == 0.0f);

		TestWorld.TickClient();
	}

	// Client is now ahead of server by 6 frames.
	SECTION("Server & client simulation, no client input")
	{
		for (int i = 0; i < 20; ++i)
		{
			TestWorld.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState->InputCounter == 0.0f);

			TestWorld.TickClient();

			const FNetPredictionTestSyncState* ClientState = TestWorld.SimObject.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ClientState->AutoCounter == ExpectedServerAutoCounterValue + 6.0f);
			CHECK(ClientState->InputCounter == 0.0f);
		}
	}

	SECTION("Server & client simulation, with client input")
	{
		TestWorld.SimObject.ClientObject.bInputPressed = true;
		float ExpectedClientInputCounterValue = 0.0f;
		float ExpectedServerInputCounterValue = 0.0f;

		for (int i = 0; i < 20; ++i)
		{
			TestWorld.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			if (i >= 6)
			{
				ExpectedServerInputCounterValue += 1.0f;
			}

			ServerState = TestWorld.SimObject.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState->InputCounter == ExpectedServerInputCounterValue);

			TestWorld.TickClient();
			ExpectedClientInputCounterValue += 1.0f;

			const FNetPredictionTestSyncState* ClientState = TestWorld.SimObject.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ClientState->AutoCounter == ExpectedServerAutoCounterValue + 6.0f);
			CHECK(ClientState->InputCounter == ExpectedClientInputCounterValue);
		}
	}

}

}