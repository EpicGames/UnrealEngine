// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetPredictionTestDriver.h"
#include "NetPredictionTestWorld.h"
#include "NetworkPredictionDriver.h"
#include "NetworkPredictionProxy.h"
#include "NetworkPredictionSerialization.h"

#include <catch2/catch_test_macros.hpp>

namespace UE::Net::Private
{

struct FTestPossession
{
	FNetPredictionTestWorld ServerWorld;
	FNetPredictionTestWorld ClientWorld;

	FNetPredictionTestObject Object1;
	FNetPredictionTestObject Object2;

	FTestPossession(UNetworkPredictionSettingsObject* Settings)
		: ServerWorld(Settings, NM_DedicatedServer)
		, ClientWorld(Settings, NM_Client)
		, Object1(ServerWorld.WorldManager, ClientWorld.WorldManager, ROLE_AutonomousProxy)
		, Object2(ServerWorld.WorldManager, ClientWorld.WorldManager, ROLE_SimulatedProxy)
	{
	}

	void TickServer()
	{
		ServerWorld.PreTick(FixedFrameRate);
		Object1.ServerObject.ReceiveServerRPCs();
		Object2.ServerObject.ReceiveServerRPCs();
		ServerWorld.Tick(FixedFrameRate);
		Object1.ServerSend();
		Object2.ServerSend();
	}

	void TickClient()
	{
		ClientWorld.PreTick(FixedFrameRate);
		Object1.ClientReceive();
		Object2.ClientReceive();
		ClientWorld.Tick(FixedFrameRate);
	}

private:
	static constexpr int32 FixedFrameRate = 60;
};

// This case includes a repro for UE-170936, disabled until it's fixed.
TEST_CASE("NetworkPrediction interpolation mode, one simulated proxy", "[possession]")
{
	// Disabling time dilation for this test since it dynamically affects how far ahead the client gets. Restored at end.
	const bool bDefaultEnableTimeDilation = NetworkPredictionCVars::EnableTimeDilation->GetBool();
	NetworkPredictionCVars::EnableTimeDilation->Set(false);
	ON_SCOPE_EXIT{ NetworkPredictionCVars::EnableTimeDilation->Set(bDefaultEnableTimeDilation); };

	UNetworkPredictionSettingsObject* Settings = NewObject<UNetworkPredictionSettingsObject>();
	Settings->Settings.SimulatedProxyNetworkLOD = ENetworkLOD::Interpolated;

	FTestPossession Worlds(Settings);
		
	Worlds.Object1.ServerObject.DebugName = TEXT("Obj1Server");
	Worlds.Object1.ClientObject.DebugName = TEXT("Obj1Client");
	Worlds.Object2.ServerObject.DebugName = TEXT("Obj2Server");
	Worlds.Object2.ClientObject.DebugName = TEXT("Obj2Client");

	float ExpectedServerAutoCounterValue = 0.0f;

	const FNetPredictionTestSyncState* ServerState = nullptr;

	// Interpolated objects will buffer FNetworkPredictionSettings::FixedTickInterpolationBufferedMS
	// frames worth of updates. For this test with the default of 100ms and fixed 60hz tick that's 6 frames.
	constexpr int32 ExpectedInterpolateBufferFrames = 6;

	SECTION("One simulated proxy")
	{
		// Run some frames to allow the client to buffer interpolation data
		for (int i = 0; i < 20; ++i)
		{
			Worlds.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState = Worlds.Object2.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState->InputCounter == 0.0f);

			Worlds.TickClient();
		}

		// Client has now buffered and is behind the server by 6 frames.
		const FNetPredictionTestSyncState* ClientState = Worlds.Object2.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
		CHECK(ClientState->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
		CHECK(ClientState->InputCounter == 0.0f);

		// Run some more frames
		for (int i = 0; i < 20; ++i)
		{
			Worlds.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState = Worlds.Object2.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState->InputCounter == 0.0f);

			Worlds.TickClient();
			ClientState = Worlds.Object2.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ClientState->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
			CHECK(ClientState->InputCounter == 0.0f);
		}
	}
	
	SECTION("Changing possession")
	{
		// This section repros UE-170936
		const FNetPredictionTestSyncState* ServerState1 = nullptr;
		const FNetPredictionTestSyncState* ServerState2 = nullptr;

		constexpr int32 ExpectedForwardPredictFrames = 6;

		// Run some frames to allow the client to predict ahead (autonomous proxy)
		// and buffer interpolation data (simulated proxy)
		for (int i = 0; i < 20; ++i)
		{
			Worlds.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState1 = Worlds.Object1.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState1->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState1->InputCounter == 0.0f);

			ServerState2 = Worlds.Object2.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState2->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState2->InputCounter == 0.0f);

			Worlds.TickClient();
		}

		const FNetPredictionTestSyncState* ClientState1 = Worlds.Object1.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
		const FNetPredictionTestSyncState* ClientState2 = Worlds.Object2.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
		// Auto proxy is ahead of the server by 6 frames.
		CHECK(ClientState1->AutoCounter == ExpectedServerAutoCounterValue + ExpectedForwardPredictFrames);
		CHECK(ClientState1->InputCounter == 0.0f);
		// Simulated proxy is behind the server by 6 frames.
		CHECK(ClientState2->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
		CHECK(ClientState2->InputCounter == 0.0f);

		// "Unpossess" the current autonomous proxy on server & client
		Worlds.Object1.ServerObject.Proxy.InitForNetworkRole(ROLE_Authority, false);
		Worlds.Object1.ClientObject.Proxy.InitForNetworkRole(ROLE_SimulatedProxy, false);

		float ExpectedClientAutoCounterValue = 21.0f;

		// Continue running
		for (int i = 0; i < 80; ++i)
		{
			Worlds.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState1 = Worlds.Object1.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState1->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState1->InputCounter == 0.0f);

			ServerState2 = Worlds.Object2.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState2->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState2->InputCounter == 0.0f);

			Worlds.TickClient();

			// Object1 went from autonomous proxy to simulated proxy
			ClientState1 = Worlds.Object1.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			// For the first few frames it's buffering interpolation data
			if (i < ExpectedInterpolateBufferFrames)
			{
				CHECK(ExpectedClientAutoCounterValue == 21.0f);
			}
			else
			{
				CHECK(ClientState1->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
			}
			CHECK(ClientState1->InputCounter == 0.0f);
			
			// Object2 is still a simulated proxy
			ClientState2 = Worlds.Object2.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ClientState2->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
			CHECK(ClientState2->InputCounter == 0.0f);
		}

		// "Possess" the other object
		Worlds.Object2.ServerObject.Proxy.InitForNetworkRole(ROLE_Authority, true);
		Worlds.Object2.ClientObject.Proxy.InitForNetworkRole(ROLE_AutonomousProxy, true);

		// Continue running
		for (int i = 0; i < 80; ++i)
		{
			Worlds.TickServer();
			ExpectedServerAutoCounterValue += 1.0f;

			ServerState1 = Worlds.Object1.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState1->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState1->InputCounter == 0.0f);

			ServerState2 = Worlds.Object2.ServerObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ServerState2->AutoCounter == ExpectedServerAutoCounterValue);
			CHECK(ServerState2->InputCounter == 0.0f);

			Worlds.TickClient();

			// Object1 is still a simulated proxy
			ClientState1 = Worlds.Object1.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			CHECK(ClientState1->AutoCounter == ExpectedServerAutoCounterValue - ExpectedInterpolateBufferFrames);
			CHECK(ClientState1->InputCounter == 0.0f);

			// Object2 is now an autonomous proxy
			ClientState2 = Worlds.Object2.ClientObject.Proxy.ReadSyncState<FNetPredictionTestSyncState>();
			if (i == 0)
			{
				// For the first client tick after "possession" the client hasn't run forward yet
				CHECK(ClientState2->AutoCounter == ExpectedServerAutoCounterValue);
			}
			else
			{
				// Now the client is a few frames ahead of the server
				CHECK(ClientState2->AutoCounter == ExpectedServerAutoCounterValue + ExpectedForwardPredictFrames);
			}
			CHECK(ClientState2->InputCounter == 0.0f);
		}
	}
}

}